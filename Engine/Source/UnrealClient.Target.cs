// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

public class UnrealClientTarget : TargetRules
{
    public UnrealClientTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Client;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		BuildEnvironment = TargetBuildEnvironment.Shared;
		ExtraModuleNames.Add("UnrealGame");
	}
}
