// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

public class UnrealEditorTarget : TargetRules
{
	public UnrealEditorTarget( TargetInfo Target ) : base(Target)
	{
		Type = TargetType.Editor;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		BuildEnvironment = TargetBuildEnvironment.Shared;
		bBuildAllModules = true;
		ExtraModuleNames.Add("UnrealGame");
	}
}
