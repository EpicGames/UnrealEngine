// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class UnrealGameTarget : TargetRules
{
	public UnrealGameTarget( TargetInfo Target ) : base(Target)
	{
		Type = TargetType.Game;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		BuildEnvironment = TargetBuildEnvironment.Shared;

		ExtraModuleNames.Add("UnrealGame");
	}
}
