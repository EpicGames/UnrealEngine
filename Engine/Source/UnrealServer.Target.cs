// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.Server)]
public class UnrealServerTarget : TargetRules
{
    public UnrealServerTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Server;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		BuildEnvironment = TargetBuildEnvironment.Shared;
		ExtraModuleNames.Add("UnrealGame");
	}
}
